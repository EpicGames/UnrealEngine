// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class CookedEditor : ModuleRules
{
	public CookedEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		bRequiresPlatformSDK = true;
		
		if (!Target.bCompileAgainstEngine)
		{
			throw new BuildException("CookedEditor module is meant for cooking only operations, and currently requires Engine to be enabled. This module is being included in a non-Engine-enabled target.");
		}

		PrivateDependencyModuleNames.AddRange(new string[]
			{
				"Core",
				"AssetRegistry",
				"NetCore",
				"CoreOnline",
				"CoreUObject",
				"Projects",
				"Engine",
			});
		
		PublicDependencyModuleNames.AddRange(new string[]
			{
				"TargetPlatform",
			});

		if (IsPlatformAvailable(UnrealTargetPlatform.Win64, bIgnoreSDKCheck: true))
		{
			PublicDefinitions.Add("COOKEDEDITOR_WITH_WINDOWSTARGETPLATFORM=1");
			PublicIncludePathModuleNames.Add("WindowsTargetPlatformSettings");
			PublicIncludePathModuleNames.Add("WindowsTargetPlatformControls");
		}
		else
		{
			PublicDefinitions.Add("COOKEDEDITOR_WITH_WINDOWSTARGETPLATFORM=0");
		}

		if (IsPlatformAvailable(UnrealTargetPlatform.Linux, bIgnoreSDKCheck: true))
		{
			PublicDefinitions.Add("COOKEDEDITOR_WITH_LINUXTARGETPLATFORM=1");
			PublicIncludePathModuleNames.Add("LinuxTargetPlatformSettings");
			PublicIncludePathModuleNames.Add("LinuxTargetPlatformControls");
		}
		else
		{
			PublicDefinitions.Add("COOKEDEDITOR_WITH_LINUXTARGETPLATFORM=0");
		}

		if (IsPlatformAvailable(UnrealTargetPlatform.Mac))
		{
			PublicDefinitions.Add("COOKEDEDITOR_WITH_MACTARGETPLATFORM=1");
			PublicIncludePathModuleNames.Add("MacTargetPlatformSettings");
			PublicIncludePathModuleNames.Add("MacTargetPlatformControls");
		}
		else
		{
			PublicDefinitions.Add("COOKEDEDITOR_WITH_MACTARGETPLATFORM=0");
		}
	}
}
