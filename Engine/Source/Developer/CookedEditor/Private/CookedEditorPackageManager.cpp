// Copyright Epic Games, Inc. All Rights Reserved.

#include "CookedEditorPackageManager.h"
#include "Algo/Sort.h"
#include "Algo/Unique.h"
#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "CookerSettings.h"
#include "CoreMinimal.h"
#include "Engine/AssetManager.h"
#include "GameDelegates.h"
#include "Logging/LogMacros.h"

DEFINE_LOG_CATEGORY_STATIC(LogCookedEditorTargetPlatform, Log, All)


TUniquePtr<ICookedEditorPackageManager> ICookedEditorPackageManager::FactoryForTargetPlatform(bool bIsCookedCooker)
{
	if (FGameDelegates::Get().GetCookedEditorPackageManagerFactoryDelegate().IsBound())
	{
		return FGameDelegates::Get().GetCookedEditorPackageManagerFactoryDelegate().Execute();
	}
	return TUniquePtr<ICookedEditorPackageManager>(new FIniCookedEditorPackageManager(bIsCookedCooker));
}



void ICookedEditorPackageManager::AddPackagesFromPath(TArray<FName>& Packages, const TCHAR* Path, EPackageSearchMode SearchMode) const
{
	UAssetManager& AssetManager = UAssetManager::Get();
	IAssetRegistry& AssetRegistry = AssetManager.GetAssetRegistry();

	TArray<FAssetData> AssetDatas;
	// look up the path in the asset registry, so we can use it to make sure the asset can be cooked
	AssetRegistry.GetAssetsByPath(Path, AssetDatas, SearchMode == EPackageSearchMode::Recurse, true);
	for (const FAssetData& AssetData : AssetDatas)
	{
		if (AssetData.IsUAsset() && AssetManager.VerifyCanCookPackage(nullptr, AssetData.PackageName, false))
		{
			if (AllowAssetToBeGathered(AssetData))
			{
				Packages.Add(AssetData.PackageName);
				UE_LOGF(LogCookedEditorTargetPlatform, Verbose, "  Adding asset %ls to be cooked", *AssetData.PackageName.ToString());
			}
		}
		else
		{
			UE_LOGF(LogCookedEditorTargetPlatform, Verbose, "  skipping asset package %ls", *AssetData.PackageName.ToString());
		}
	}
}

void ICookedEditorPackageManager::GatherAllPackagesExceptDisabled(TArray<FName>& PackageNames, const TArray<FString>& DisabledPlugins) const
{
	GetEnginePackagesToCook(PackageNames);
	GetProjectPackagesToCook(PackageNames);

	// copy array to set for faster contains calls
	TSet<FString> CookedEditorDisabledPlugins;
	CookedEditorDisabledPlugins.Append(DisabledPlugins);

	// walk over plugins and cook their content
	for (TSharedRef<IPlugin> Plugin : IPluginManager::Get().GetEnabledPluginsWithContent())
	{
		if (!CookedEditorDisabledPlugins.Contains(Plugin->GetName()))
		{
			// check if this engine or project plugin shuold be cooked
			bool bShouldCook = (Plugin->GetLoadedFrom() == EPluginLoadedFrom::Engine) ? AllowEnginePluginContentToBeCooked(Plugin) :
				AllowProjectPluginContentToBeCooked(Plugin);

			if (bShouldCook)
			{
				UE_LOGF(LogCookedEditorTargetPlatform, Display, "Adding enabled plugin with content: %ls", *Plugin->GetName());
				AddPackagesFromPath(PackageNames, *Plugin->GetMountedAssetPath(), EPackageSearchMode::Recurse);
				
			}
		}
	}

	FilterGatheredPackages(PackageNames);
}

void ICookedEditorPackageManager::FilterGatheredPackages(TArray<FName>& PackageNames) const
{

}



FIniCookedEditorPackageManager::FIniCookedEditorPackageManager(bool bInIsCookedCooker)
	: bIsCookedCooker(bInIsCookedCooker)
{
	EngineAssetPaths = GetConfigArray(TEXT("EngineAssetPaths"));
	ProjectAssetPaths = GetConfigArray(TEXT("ProjectAssetPaths"));
	DisallowedPathsToGather = GetConfigArray(TEXT("DisallowedPathsToGather"));
	DisabledPlugins = GetConfigArray(TEXT("DisabledPlugins"));
}

void FIniCookedEditorPackageManager::InitializeClasses()
{
	if (bClassesInitialized)
	{
		return;
	}
	bClassesInitialized = true;

	if (const IConsoleVariable* DisplayModeCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("cook.displaymode")))
	{
		CookProgressDisplayMode = DisplayModeCVar->GetInt();
	}

	IAssetRegistry& AssetRegistry = IAssetRegistry::GetChecked();
	// SearchAllAssets should be a noop because the cooker already did it, but run it just in case.
	AssetRegistry.SearchAllAssets(true /* bSynchronousSearch */);

	constexpr TCHAR WildcardCharacter = '*';

	TArray<FString> DisallowedObjectClassWildcards;
	TArray<FString> DisallowedObjectClassNamesToLoad = GetConfigArray(TEXT("DisallowedObjectClassesToLoad"));;
	for (TArray<FString>::TIterator Iter(DisallowedObjectClassNamesToLoad); Iter; ++Iter)
	{
		FString& ClassName = *Iter;
		int32 UnusedIndex;
		if (ClassName.FindChar(WildcardCharacter, UnusedIndex))
		{
			DisallowedObjectClassWildcards.Add(MoveTemp(ClassName));
			Iter.RemoveCurrentSwap();
		}
	}
	if (!DisallowedObjectClassWildcards.IsEmpty())
	{
		// Get the list of all blueprint and native classes from the AssetRegistry
		TArray<FTopLevelAssetPath> ClassNamesUObject;
		ClassNamesUObject.Add(FTopLevelAssetPath(UObject::StaticClass()));
		TSet<FTopLevelAssetPath> AllClasses;
		AssetRegistry.GetDerivedClassNames(ClassNamesUObject, TSet<FTopLevelAssetPath>(), AllClasses);

		// Check the wildcards from config against each classpath
		TSet<FString> MatchingClassPaths;
		FString ClassPathStr;
		for (FTopLevelAssetPath& ClassPath : AllClasses)
		{
			ClassPath.ToString(ClassPathStr);
			bool bMatches = false;
			for (const FString& Wildcard : DisallowedObjectClassWildcards)
			{
				if (ClassPathStr.MatchesWildcard(Wildcard, ESearchCase::IgnoreCase))
				{
					bMatches = true;
					break;
				}
			}
			if (bMatches)
			{
				DisallowedObjectClassNamesToLoad.Add(ClassPathStr);
			}
		}
	}
	Algo::Sort(DisallowedObjectClassNamesToLoad);
	DisallowedObjectClassNamesToLoad.SetNum(Algo::Unique(DisallowedObjectClassNamesToLoad));
	TArray<FTopLevelAssetPath> DisallowedObjectBaseClassPaths;
	DisallowedObjectBaseClassPaths.Reserve(DisallowedObjectClassNamesToLoad.Num());
	for (const FString& ClassName : DisallowedObjectClassNamesToLoad)
	{
		check(FPackageName::IsValidObjectPath(ClassName));
		FTopLevelAssetPath ClassPath(ClassName);
		check(ClassPath.IsValid());
		DisallowedObjectBaseClassPaths.Add(ClassPath);
	}

	DisallowedObjectClassesToLoad.Reset();
	AssetRegistry.GetDerivedClassNames(DisallowedObjectBaseClassPaths, TSet<FTopLevelAssetPath>(),
		DisallowedObjectClassesToLoad);

	TArray<FString> DisallowedAssetClassNamesToGather = GetConfigArray(TEXT("DisallowedAssetClassesToGather"));;
	for (const FString& ClassName : DisallowedAssetClassNamesToGather)
	{
		check(FPackageName::IsValidObjectPath(ClassName));
		UClass* Class = FindObject<UClass>(nullptr, *ClassName);
		check(Class);

		DisallowedAssetClassesToGather.Add(Class);
	}
}

TArray<FString> FIniCookedEditorPackageManager::GetConfigArray(const TCHAR* Key) const
{
	return GetConfigArray(Key, bIsCookedCooker);
}

TArray<FString> FIniCookedEditorPackageManager::GetConfigArray(const TCHAR* Key, bool bIsCookedCooker)
{
	const TCHAR* SharedIniSection = TEXT("CookedEditorSettings");
	const TCHAR* SpecificiIniSection = bIsCookedCooker ? TEXT("CookedEditorSettings_CookedCooker") : TEXT("CookedEditorSettings_CookedEditor");

	TArray<FString> TempArray;
	TArray<FString> ResultArray;

	GConfig->GetArray(SpecificiIniSection, Key, ResultArray, GGameIni);
	GConfig->GetArray(SharedIniSection, Key, TempArray, GGameIni);
	ResultArray.Append(TempArray);

	return ResultArray;
}

void FIniCookedEditorPackageManager::GatherAllPackages(TArray<FName>& PackageNames) const
{
	check(bClassesInitialized); // Should be set by InitializeClasses from InitializeForCook
	GatherAllPackagesExceptDisabled(PackageNames, DisabledPlugins);
}

void FIniCookedEditorPackageManager::FilterGatheredPackages(TArray<FName>& PackageNames) const
{
	// now filter based on ini settings
	PackageNames.RemoveAll([this](FName& AssetPath)
		{
			FNameBuilder AssetPathBuilder(AssetPath);
			const FStringView AssetPathView(AssetPathBuilder);
			for (const FString& Path : DisallowedPathsToGather)
			{
				if (AssetPathView.StartsWith(Path))
				{
					return true;
				}
			}
			return false;
		});
	PackageNames.Remove(NAME_None);
}

void FIniCookedEditorPackageManager::InitializeForCook()
{
	InitializeClasses();
}

void FIniCookedEditorPackageManager::GetEnginePackagesToCook(TArray<FName>& PackagesToCook) const
{
	for (const FString& Path : EngineAssetPaths)
	{
		AddPackagesFromPath(PackagesToCook, *Path, EPackageSearchMode::Recurse);
	}

	// specific assets to cook
	PackagesToCook.Append(GetConfigArray(TEXT("EngineSpecificAssetsToCook")));
}

void FIniCookedEditorPackageManager::GetProjectPackagesToCook(TArray<FName>& PackagesToCook) const
{
	for (const FString& Path : ProjectAssetPaths)
	{
		AddPackagesFromPath(PackagesToCook, *Path, EPackageSearchMode::Recurse);
	}

	// make sure editor startup map is cooked
	FString EditorStartupMap;
	if (GConfig->GetString(TEXT("/Script/EngineSettings.GameMapsSettings"), TEXT("EditorStartupMap"), EditorStartupMap, GEngineIni))
	{
		PackagesToCook.Add(*EditorStartupMap);
	}

	// specific assets to cook
	PackagesToCook.Append(GetConfigArray(TEXT("ProjectSpecificAssetsToCook")));
}

bool FIniCookedEditorPackageManager::AllowObjectToBeCooked(const class UObject* Obj) const
{
	check(bClassesInitialized); // Should be set by InitializeClasses from InitializeForCook
	const UClass* ObjAsClass = Cast<UClass>(Obj);

	// A pointer to a disallowed native class is not filtered out, only instances of the native class are
	// filtered out. For non-native classes, both the pointer to the non-native class and instances of the class
	// are filtered out.
	if (!ObjAsClass || !ObjAsClass->IsNative())
	{
		const UClass* ClassToCheck = ObjAsClass ? ObjAsClass : Obj->GetClass();
		if (DisallowedObjectClassesToLoad.Contains(ClassToCheck->GetClassPathName()))
		{
			if(CookProgressDisplayMode != (int32)ECookProgressDisplayMode::Nothing)
			{
				UE_LOGF(LogCookedEditorTargetPlatform, Log, "Object: %ls will not cook due to disallowed class: %ls", *Obj->GetFName().ToString(), *ClassToCheck->GetClassPathName().ToString());
			}
			return false;
		}
	}
	return true;
}

bool FIniCookedEditorPackageManager::AllowAssetToBeGathered(const struct FAssetData& AssetData) const
{
	check(bClassesInitialized); // Should be set by InitializeClasses from InitializeForCook
	for (UClass* Class : DisallowedAssetClassesToGather)
	{
		if (AssetData.IsInstanceOf(Class))
		{
			if (CookProgressDisplayMode != (int32)ECookProgressDisplayMode::Nothing)
			{
				UE_LOGF(LogCookedEditorTargetPlatform, Log, "Asset: %ls will not be gathered due to disallowed class: %ls", *AssetData.PackageName.ToString(), *Class->GetClassPathName().ToString());
			}
			return false;
		}
	}

	return true;
}

bool FIniCookedEditorPackageManager::AllowEnginePluginContentToBeCooked(const TSharedRef<IPlugin>) const
{
	return true;
}

bool FIniCookedEditorPackageManager::AllowProjectPluginContentToBeCooked(const TSharedRef<IPlugin>) const
{
	return true;
}
