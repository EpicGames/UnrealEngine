// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// IWYU pragma: begin_exports

#include "CoreMinimal.h"
#include "Common/TargetPlatformBase.h"
#include "Interfaces/IPluginManager.h"

#if COOKEDEDITOR_WITH_WINDOWSTARGETPLATFORM
	#include "GenericWindowsTargetPlatformControls.h"
	#include "Windows/WindowsPlatformProperties.h"
#endif

#if COOKEDEDITOR_WITH_LINUXTARGETPLATFORM
	#include "LinuxTargetPlatformSettings.h"
	#include "LinuxTargetPlatformControls.h"
#endif

#if COOKEDEDITOR_WITH_MACTARGETPLATFORM
	#include "GenericMacTargetPlatformSettings.h"
	#include "GenericMacTargetPlatformControls.h"
#endif

#if COOKEDEDITOR_WITH_WINDOWSTARGETPLATFORM
	typedef TGenericWindowsTargetPlatformSettings<FWindowsPlatformProperties<false, false, false>> FWindowsEditorTargetPlatformSettingsParent;
	typedef TGenericWindowsTargetPlatformControls<FWindowsPlatformProperties<false, false, false>> FWindowsEditorTargetPlatformControlsParent;
#endif

#if COOKEDEDITOR_WITH_LINUXTARGETPLATFORM
typedef TLinuxTargetPlatformSettings<FLinuxPlatformProperties<false, false, false, false>> FLinuxEditorTargetPlatformSettingsParent;
typedef TLinuxTargetPlatformControls<FLinuxPlatformProperties<false, false, false, false>> FLinuxEditorTargetPlatformControlsParent;
#endif

#if COOKEDEDITOR_WITH_MACTARGETPLATFORM
typedef TGenericMacTargetPlatformSettings<false, false, false> FMacEditorTargetPlatformSettingsParent;
typedef TGenericMacTargetPlatformControls<false, false, false> FMacEditorTargetPlatformControlsParent;
#endif

#if PLATFORM_WINDOWS
typedef FWindowsEditorTargetPlatformSettingsParent FHostPlatformEditorTargetPlatformSettingsParent;
typedef FWindowsEditorTargetPlatformControlsParent FHostPlatformEditorTargetPlatformControlsParent;
#elif PLATFORM_LINUX
typedef FLinuxEditorTargetPlatformSettingsParent FHostPlatformEditorTargetPlatformSettingsParent;
typedef FLinuxEditorTargetPlatformControlsParent FHostPlatformEditorTargetPlatformControlsParent;
#elif PLATFORM_MAC
typedef FMacEditorTargetPlatformSettingsParent FHostPlatformEditorTargetPlatformSettingsParent;
typedef FMacEditorTargetPlatformControlsParent FHostPlatformEditorTargetPlatformControlsParent;
#endif

// IWYU pragma: end_exports

/**
 * Allows a project to control how packages are cooked when making a cooked editor.
 * 
 * Unlike a runtime game, a cooked editor typically needs to include all engine and plugin content, to make all 
 * resources available for the crafting of future games in the cooked editor with asset needs that are not yet known. 
 * When cooking a cooked editor target platform, an instance of this class adds the requests for all of that extra content.
 *
 * The default implementation is FIniCookedEditorPackageManager which allows for Game.ini settings to control most 
 * functionality. By default (as set in BaseGame.ini) all content is cooked with a few class types and plugns that 
 * are not due to incompatibility with having been cooked and loaded by the editor.
 * 
 * If "Cook Against Release" mode is enabled, then making the cooked editor will act like "DLC" where it uses an already
 * created Game or Client release and will not cook pacakges that are already in the release. 
 * 
 * To override this class, generally you will want to override the Project functions in the first public section.
 * In GetProjectPackagesToCook(), you can call AddPackagesFromPath to cook spefic project content subdirectories as needed
 * 
 * You likely can use just this class without needing to subclass TCookedEditorTargetPlatform at all, but you can do that
 * if needed.
 */
class ICookedEditorPackageManager
{
public:

	/** 
	 * Construct a package manager for the given TP
	 */
	static COOKEDEDITOR_API TUniquePtr<ICookedEditorPackageManager> FactoryForTargetPlatform(bool bIsCookedCooker);

	virtual ~ICookedEditorPackageManager() = default;

	/**
	 * Hook called when the manager's TargetPlatform is selected for cooking. Load any required assets or data.
	 * May be called multiple times per process; it is called each time a cook starts for the platform.
	 */
	virtual void InitializeForCook() = 0;

	/**
	 * Override to control engine packages to cook
	 */
	virtual void GetEnginePackagesToCook(TArray<FName>& PackagesToCook) const = 0;

	/**
	 * Override to add project specific packages to cook
	 */
	virtual void GetProjectPackagesToCook(TArray<FName>& PackagesToCook) const = 0;

	/**
	 * Allows the subclass to exclude specific objects in packages discovered during the cook.
	 * By default, will disallow UBlueprint objects
	 */
	virtual bool AllowObjectToBeCooked(const class UObject* Obj) const = 0;

	/**
	 * Allows the subclass to disallow some assets to be gathered during AddPackagesFromPath, for instance 
	 * This is similar to AllowObjectToBeCooked, but it can help with, for instance, removing maps from
	 * subdirectories, while still allowing the normal maps that come from the standard cooking process
	 */
	virtual bool AllowAssetToBeGathered(const struct FAssetData& AssetData) const = 0;

	/**
	 * Should the given enabled engine plugin be cooked?
	 */
	virtual bool AllowEnginePluginContentToBeCooked(const TSharedRef<IPlugin>) const = 0;

	/**
	 * Should the given enabled project plugin be cooked?
	 */
	virtual bool AllowProjectPluginContentToBeCooked(const TSharedRef<IPlugin>) const = 0;

	/**
	 * Gathers the packages this PackageManager wants to manage (ie cook)
	 */
	virtual void GatherAllPackages(TArray<FName>& PackageNames) const = 0;

protected:

	enum class EPackageSearchMode
	{
		TopDirectoryOnly,
		Recurse,
	};

	/**
	 * Use the asset registry to find packages to be cooked
	 * @param Packages output array of paths
	 * @param Path rooted path to look in ("/Engine", etc)
	 * @param SeachMode Recurse or TopDirectoryOnly
	 */
	COOKEDEDITOR_API void AddPackagesFromPath(TArray<FName>& Packages, const TCHAR* Path, EPackageSearchMode SearchMode) const;

	/**
	 * Allow a subclass to remove packages found in GatherAllPackages
	 */
	COOKEDEDITOR_API virtual void FilterGatheredPackages(TArray<FName>& PackageNames) const;

protected:

	/**
	 * Meat of this class, this calls other functions that generally will be overridden - subclass needs to pass in disabled plugins
	 */
	COOKEDEDITOR_API void GatherAllPackagesExceptDisabled(TArray<FName>& PackageNames, const TArray<FString>& DisabledPlugins) const;
};



/**
 * An implentation of ICookedEditorPackageManager that uses BaseGame.ini / DefaultGame.ini to control cooked editor packaging setup
 * 
 */
class FIniCookedEditorPackageManager : public ICookedEditorPackageManager
{
	TArray<FString> EngineAssetPaths;
	TArray<FString> ProjectAssetPaths;
	TArray<FString> DisabledPlugins;
	TSet<FTopLevelAssetPath> DisallowedObjectClassesToLoad;
	TArray<UClass*> DisallowedAssetClassesToGather;
	TArray<FString> DisallowedPathsToGather;

	// true if this is a cooked cooker (false for cooker editor)
	bool bIsCookedCooker;

	// Guard to prevent multiple InitializeClasses calls
	bool bClassesInitialized = false;

	// Used to control logging of certain warnings.
	int32 CookProgressDisplayMode = 0;

	// gets an array from two sections, depending on bIsCookedCooker setting
	COOKEDEDITOR_API TArray<FString> GetConfigArray(const TCHAR* Key) const;

	// Load the classes specified from config settings that we need to reference
	COOKEDEDITOR_API void InitializeClasses();
public:

	COOKEDEDITOR_API FIniCookedEditorPackageManager(bool bIsCookedCooker);

	// gets an array from two sections, depending on bIsCookedCooker setting
	static COOKEDEDITOR_API TArray<FString> GetConfigArray(const TCHAR* Key, bool bIsCookedCooker);

	COOKEDEDITOR_API virtual void FilterGatheredPackages(TArray<FName>& PackageNames) const override;
	COOKEDEDITOR_API virtual void InitializeForCook() override;
	COOKEDEDITOR_API virtual void GetEnginePackagesToCook(TArray<FName>& PackagesToCook) const override;
	COOKEDEDITOR_API virtual void GetProjectPackagesToCook(TArray<FName>& PackagesToCook) const override;
	COOKEDEDITOR_API virtual bool AllowObjectToBeCooked(const class UObject* Obj) const override;
	COOKEDEDITOR_API virtual bool AllowAssetToBeGathered(const struct FAssetData& AssetData) const override;
	COOKEDEDITOR_API virtual bool AllowEnginePluginContentToBeCooked(const TSharedRef<IPlugin>) const override;
	COOKEDEDITOR_API virtual bool AllowProjectPluginContentToBeCooked(const TSharedRef<IPlugin>) const override;
	
	COOKEDEDITOR_API virtual void GatherAllPackages(TArray<FName>& PackageNames) const override;
};
