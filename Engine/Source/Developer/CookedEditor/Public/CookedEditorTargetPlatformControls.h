// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Common/TargetPlatformControlsBase.h"
#include "Interfaces/IPluginManager.h"
#include "CookedEditorPackageManager.h"

/**
 * Templated class for a target platform used to cook a cooked editor. It needs to inherit from a 
 * desktop platform's target platform such as TGenericWindowsTargetPlatformControls. See
 * CookedEditorPackageManager.h for some typedef'd standard base classes that can be used.
 * 
 * The majority of the functionality is performed in an instance of a ICookedEditorPackageManager subclass.
 * See that class for more information.
 */
template<typename Base>
class TCookedEditorTargetPlatformControls : public Base
{
public:

	TCookedEditorTargetPlatformControls(ITargetPlatformSettings* TargetPlatformSettings)
		:Base(TargetPlatformSettings)
	{
		PackageManager = ICookedEditorPackageManager::FactoryForTargetPlatform(false);
	}

	/** Allows for a custom target platform module to initialize this TargetPlatform with an existing
	 * PackageManager instead of going through the standard factory function */
	explicit TCookedEditorTargetPlatformControls(TUniquePtr<ICookedEditorPackageManager>&& ExistingManager, ITargetPlatformSettings* TargetPlatformSettings)
		:Base(TargetPlatformSettings)
	{
		PackageManager = MoveTemp(ExistingManager);
	}

	~TCookedEditorTargetPlatformControls() = default;

	virtual FString PlatformName() const override
	{
		static FString CachedPlatformName = this->GetTargetPlatformSettings()->IniPlatformName() + TEXT("CookedEditor");
		return CachedPlatformName;
	}

	virtual FString CookingDeviceProfileName() const override
	{
		return Base::PlatformName();
	}

	/**
	 * If you override this to return false, you will have to stage uncooked assets to allow the editor to run properly
	 */
	virtual bool AllowsEditorObjects() const override
	{
		return true;
	}

	virtual bool AllowObject(const class UObject* Obj) const override
	{
		// probably don't need this check, but it can't hurt
		if (!AllowsEditorObjects())
		{
			return true;
		}

		return PackageManager->AllowObjectToBeCooked(Obj);
	}

	virtual void InitializeForCook() override
	{
		PackageManager->InitializeForCook();
	}

	virtual void GetExtraPackagesToCook(TArray<FName>& PackageNames) const override
	{
		if (AllowsEditorObjects())
		{
			PackageManager->GatherAllPackages(PackageNames);
		}
	}

	virtual float GetVariantPriority() const override
	{
		// by returning -1, we will never use this variant when targeting host platform this class implements
		// (without this, cooking for Windows in the editor may choose this variant to cook for, which we never want)
		return -1.0f;
	}

	virtual void GetAllDevices(TArray<ITargetDevicePtr>& OutDevices) const override
	{
	}

	virtual ITargetDevicePtr GetDefaultDevice() const override
	{
		return nullptr;
	}


protected:

	TUniquePtr<ICookedEditorPackageManager> PackageManager;

};


template<typename Base>
class TCookedCookerTargetPlatformControls : public Base
{
public:

	TCookedCookerTargetPlatformControls(ITargetPlatformSettings* TargetPlatformSettings)
		: Base(TargetPlatformSettings)
	{
		PackageManager = ICookedEditorPackageManager::FactoryForTargetPlatform(true);
	}

	TCookedCookerTargetPlatformControls(TUniquePtr<ICookedEditorPackageManager>&& ExistingManager, ITargetPlatformSettings* TargetPlatformSettings)
		: Base(TargetPlatformSettings)
	{
		PackageManager = MoveTemp(ExistingManager);
	}

	~TCookedCookerTargetPlatformControls() = default;


	virtual FString PlatformName() const override
	{
		static FString CachedPlatformName = this->GetTargetPlatformSettings()->IniPlatformName() + TEXT("CookedCooker");
		return CachedPlatformName;
	}

	virtual FString CookingDeviceProfileName() const override
	{
		return Base::PlatformName();
	}

	virtual bool AllowsEditorObjects() const override
	{
		return true;
	}

	virtual bool AllowsDevelopmentObjects() const override
	{
		return false;
	}

	virtual bool AllowObject(const UObject* Obj) const override
	{
		// probably don't need this check, but it can't hurt
		if (!AllowsEditorObjects())
		{
			return true;
		}

		return PackageManager->AllowObjectToBeCooked(Obj);
	}
	
	virtual void InitializeForCook() override
	{
		PackageManager->InitializeForCook();
	}

	virtual void GetExtraPackagesToCook(TArray<FName>& PackageNames) const override
	{
		if (AllowsEditorObjects())
		{
			PackageManager->GatherAllPackages(PackageNames);
		}
	}


	//////////////////////////////////////////////////////////////////
	// Disabling stuff since it's just a cooker
	//////////////////////////////////////////////////////////////////


	virtual void GetTextureFormats(const class UTexture* InTexture, TArray< TArray<FName> >& OutFormats) const override
	{
		// no textures please
	}

	virtual bool SupportsVariants() const override
	{
		return false;
	}

	virtual bool AllowAudioVisualData() const override
	{
		return false;
	}

	virtual float GetVariantPriority() const override
	{
		// by returning -1, we will never use this variant when targeting host platform this class implements
		// (without this, cooking for Windows in the editor may choose this variant to cook for, which we never want)
		return -1.0f;
	}

	virtual void GetAllDevices(TArray<ITargetDevicePtr>& OutDevices) const override
	{
	}

	virtual ITargetDevicePtr GetDefaultDevice() const override
	{
		return nullptr;
	}



	TUniquePtr<ICookedEditorPackageManager> PackageManager;

};
