// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Common/TargetPlatformSettingsBase.h"
#include "Interfaces/IPluginManager.h"
#include "CookedEditorPackageManager.h"

/**
 * Templated class for a target platform used to cook a cooked editor. It needs to inherit from a 
 * desktop platform's target platform such as TGenericWindowsTargetPlatformSettings. See
 * CookedEditorPackageManager.h for some typedef'd standard base classes that can be used.
 * 
 * The majority of the functionality is performed in an instance of a ICookedEditorPackageManager subclass.
 * See that class for more information.
 */
template<typename Base>
class TCookedEditorTargetPlatformSettings : public Base
{
public:

	TCookedEditorTargetPlatformSettings()
	{
		PackageManager = ICookedEditorPackageManager::FactoryForTargetPlatform(false);
	}

	/** Allows for a custom target platform module to initialize this TargetPlatform with an existing
	 * PackageManager instead of going through the standard factory function */
	explicit TCookedEditorTargetPlatformSettings(TUniquePtr<ICookedEditorPackageManager>&& ExistingManager)
	{
		PackageManager = MoveTemp(ExistingManager);
	}

	~TCookedEditorTargetPlatformSettings() = default;

	virtual void GetReflectionCaptureFormats(TArray<FName>& OutFormats) const override
	{
		Base::GetReflectionCaptureFormats(OutFormats);

		// UMapBuildDataRegistry::PostLoad() assumes that editor always needs encoded data, so when cooking for the editor,
		// make sure that the EncodedHDR format is included
		OutFormats.AddUnique(TEXT("EncodedHDR"));
	}



protected:

	TUniquePtr<ICookedEditorPackageManager> PackageManager;

};


template<typename Base>
class TCookedCookerTargetPlatformSettings : public Base
{
public:

	TCookedCookerTargetPlatformSettings()
	{
		PackageManager = ICookedEditorPackageManager::FactoryForTargetPlatform(true);
	}

	TCookedCookerTargetPlatformSettings(TUniquePtr<ICookedEditorPackageManager>&& ExistingManager)
	{
		PackageManager = MoveTemp(ExistingManager);
	}

	~TCookedCookerTargetPlatformSettings() = default;
	
	//////////////////////////////////////////////////////////////////
	// Disabling stuff since it's just a cooker
	//////////////////////////////////////////////////////////////////

	virtual void GetAllPossibleShaderFormats(TArray<FName>& OutFormats) const override
	{
		// no shaders please
	}

	virtual void GetAllTargetedShaderFormats(TArray<FName>& OutFormats) const override
	{
		// no shaders please
	}

	virtual bool SupportsFeature(ETargetPlatformFeatures Feature) const override
	{
		switch (Feature)
		{
		case ETargetPlatformFeatures::AudioStreaming:
		case ETargetPlatformFeatures::MemoryMappedAudio:
			return false;

		case ETargetPlatformFeatures::CanCookPackages:
		case ETargetPlatformFeatures::Packaging:
			return true;

		default:
			return Base::SupportsFeature(Feature);
		}
	}

	TUniquePtr<ICookedEditorPackageManager> PackageManager;

};
